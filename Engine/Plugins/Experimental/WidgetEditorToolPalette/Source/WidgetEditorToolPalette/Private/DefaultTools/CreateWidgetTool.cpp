// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultTools/CreateWidgetTool.h"
#include "InteractiveToolManager.h"
#include "WidgetModeManager.h"
#include "ToolBuilderUtil.h"
#include "WidgetBlueprintEditor.h"
#include "Tools/AssetEditorContextInterface.h"
#include "ContextObjectStore.h"
#include "Engine/World.h"

#include "WidgetBlueprint.h"
#include "WidgetTemplate.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "WidgetBlueprintEditorUtils.h"
#include "Blueprint/WidgetTree.h"
#include "Components/Button.h"
#include "Components/Widget.h"
#include "Templates/WidgetTemplateClass.h"
#include "DragDrop/WidgetTemplateDragDropOp.h"
#include "Designer/SDesignerView.h"

// localization namespace
#define LOCTEXT_NAMESPACE "UCreateWidgetTool"

/*
 * ToolBuilder
 */

UCreateWidgetToolBuilder::UCreateWidgetToolBuilder()
	: WidgetClass(nullptr)
{
}

UInteractiveTool* UCreateWidgetToolBuilder::BuildTool(const FToolBuilderState & SceneState) const
{
	UCreateWidgetTool* NewTool = NewObject<UCreateWidgetTool>(SceneState.ToolManager);
	NewTool->WidgetClass = WidgetClass;

	if (IAssetEditorContextInterface* AssetEditorContext = SceneState.ToolManager->GetContextObjectStore()->FindContext<IAssetEditorContextInterface>())
	{
		if (const IToolkitHost* ToolkitHost = AssetEditorContext->GetToolkitHost())
		{
			// @TODO: DarenC - try to avoid this cast
			FWidgetModeManager& WidgetModeManager = static_cast<FWidgetModeManager&>(ToolkitHost->GetEditorModeManager());
			NewTool->SetOwningWidget(WidgetModeManager.GetManagedWidget());
			NewTool->SetOwningToolkit(WidgetModeManager.OwningToolkit.Pin());
		}
	}

	return NewTool;
}

/*
 * Tool
 */

UCreateWidgetToolProperties::UCreateWidgetToolProperties()
{
	// PlaceOnObjects = true;
}

UCreateWidgetTool::UCreateWidgetTool()
{
}

void UCreateWidgetTool::SetOwningToolkit(TSharedPtr<IToolkit> InOwningToolkit)
{
	OwningToolkit = StaticCastSharedPtr<FWidgetBlueprintEditor>(InOwningToolkit);
}

void UCreateWidgetTool::SetOwningWidget(TSharedPtr<SWidget> InOwningWidget)
{
	OwningWidget = StaticCastSharedPtr<SDesignerView>(InOwningWidget);
}

void UCreateWidgetTool::Setup()
{
	USingleClickTool::Setup();

	Properties = NewObject<UCreateWidgetToolProperties>(this);
	AddToolPropertySource(Properties);
}

bool UCreateWidgetTool::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (TSharedPtr<FWidgetBlueprintEditor> BPEditor = OwningToolkit.Pin())
	{
		if (TSharedPtr<SDesignerView> PinnedOwningWidget = OwningWidget.Pin())
		{
			const FVector2D ClickPosition = PinnedOwningWidget->PanelCoordToGraphCoord(MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()));

			// Fire drag drop event at the click location
			TSharedPtr<FWidgetTemplateClass> WidgetTemplate = MakeShared<FWidgetTemplateClass>(WidgetClass);
			TSharedRef<FWidgetTemplateDragDropOp> TemplateDragDrop = FWidgetTemplateDragDropOp::New(WidgetTemplate);
			FDragDropEvent DragDropEvent = FDragDropEvent(MouseEvent, TemplateDragDrop);

			PinnedOwningWidget->OnDrop(MyGeometry, DragDropEvent).IsEventHandled();
			
			// We only create one widget per click, activate default select tool after click.
			GetToolManager()->SelectActiveToolType(EToolSide::Mouse, "DefaultCursor");
			GetToolManager()->ActivateTool(EToolSide::Mouse);
		}
	}

	// Always return not handled, we want to use the default mouse up FReply, which includes changing mouse capture.
	return false;
}

#undef LOCTEXT_NAMESPACE