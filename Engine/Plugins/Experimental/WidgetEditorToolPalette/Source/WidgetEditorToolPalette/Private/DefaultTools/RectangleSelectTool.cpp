// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultTools/RectangleSelectTool.h"
#include "InteractiveToolManager.h"
#include "WidgetModeManager.h"
#include "ToolBuilderUtil.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "Tools/AssetEditorContextInterface.h"
#include "ContextObjectStore.h"
#include "MarqueeOperation.h"
#include "Designer/SDesignerView.h"
#include "WidgetBlueprintEditor.h"
#include "WidgetBlueprint.h"
#include "WidgetReference.h"
#include "Blueprint/WidgetTree.h"
#include "Styling/AppStyle.h"

// localization namespace
#define LOCTEXT_NAMESPACE "URectangleSelectTool"

/*
 * ToolBuilder
 */

bool URectangleSelectToolBuilder::CanBuildTool(const FToolBuilderState & SceneState) const
{
	return true;
}

UInteractiveTool* URectangleSelectToolBuilder::BuildTool(const FToolBuilderState & SceneState) const
{
	URectangleSelectTool* NewTool = NewObject<URectangleSelectTool>(SceneState.ToolManager);

	if (IAssetEditorContextInterface* AssetEditorContext = SceneState.ToolManager->GetContextObjectStore()->FindContext<IAssetEditorContextInterface>())
	{
		if (const IToolkitHost* ToolkitHost = AssetEditorContext->GetToolkitHost())
		{
			// @TODO: DarenC - try to avoid this cast
			FWidgetModeManager& WidgetModeManager = static_cast<FWidgetModeManager&>(ToolkitHost->GetEditorModeManager());
			NewTool->SetOwningWidget(WidgetModeManager.GetManagedWidget());
			NewTool->SetOwningToolkit(WidgetModeManager.OwningToolkit.Pin());
		}
	}

	return NewTool;
}

/*
 * Tool
 */

URectangleSelectProperties::URectangleSelectProperties()
	: UInteractiveToolPropertySet()
	, bIsMarqueeActive(false)
{
}

void URectangleSelectTool::SetOwningToolkit(TSharedPtr<IToolkit> InOwningToolkit)
{
	OwningToolkit = StaticCastSharedPtr<FWidgetBlueprintEditor>(InOwningToolkit);
}

void URectangleSelectTool::SetOwningWidget(TSharedPtr<SWidget> InOwningWidget)
{
	OwningWidget = StaticCastSharedPtr<SDesignerView>(InOwningWidget);
}

void URectangleSelectTool::Setup()
{
	UInteractiveTool::Setup();

	// Create the property set and register it with the Tool
	Properties = NewObject<URectangleSelectProperties>(this, "Rectangle Select");
	AddToolPropertySource(Properties);
}

bool URectangleSelectTool::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	// The existing marquee operation was canceled
	if (InKeyEvent.GetKey() == EKeys::Escape && Properties->Marquee.IsValid())
	{
		Properties->Marquee = FMarqueeOperation();
		Properties->bIsMarqueeActive = false;

		return true;
	}

	return false;
}

bool URectangleSelectTool::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (TSharedPtr<SDesignerView> PinnedOwningWidget = OwningWidget.Pin())
		{
			const FVector2D ClickPosition = PinnedOwningWidget->PanelCoordToGraphCoord(MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()));

			Properties->Marquee = FMarqueeOperation();
			Properties->Marquee.Start(ClickPosition, FMarqueeOperation::Type::Add);
			Properties->bIsMarqueeActive = true;
			PrevSelectionEndPoint = ClickPosition;

			// Clear existing selection
			if (!MouseEvent.GetModifierKeys().IsControlDown())
			{
				if (TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = OwningToolkit.Pin())
				{
					WidgetBlueprintEditor->SelectWidgets({}, false);
				}
			}

			return true;
		}
	}

	return false;
}

bool URectangleSelectTool::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (Properties->bIsMarqueeActive)
		{
			if (TSharedPtr<SDesignerView> PinnedOwningWidget = OwningWidget.Pin())
			{
				const FVector2D Position = PinnedOwningWidget->PanelCoordToGraphCoord(MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()));

				// Just test equality assuming whole pixel movements
				if (Position != Properties->Marquee.Rect.EndPoint)
				{
					Properties->Marquee.Rect.UpdateEndPoint(Position);
					SelectWidgetsAffectedByMarquee();
				}
			}

			// The existing marquee operation has finished
			Properties->Marquee = FMarqueeOperation();
			Properties->bIsMarqueeActive = false;

			return true;
		}
	}

	return false;
}

bool URectangleSelectTool::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (Properties->bIsMarqueeActive)
	{
		if (TSharedPtr<SDesignerView> PinnedOwningWidget = OwningWidget.Pin())
		{
			const FVector2D Position = PinnedOwningWidget->PanelCoordToGraphCoord(MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()));

			// Just test equality assuming whole pixel movements
			if (Position != Properties->Marquee.Rect.EndPoint)
			{
				Properties->Marquee.Rect.UpdateEndPoint(Position);
				SelectWidgetsAffectedByMarquee();
			}
		}

		return true;
	}

	return false;
}

int32 URectangleSelectTool::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	if (TSharedPtr<SDesignerView> PinnedOwningWidget = OwningWidget.Pin())
	{
		if (Properties->Marquee.IsValid())
		{
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(Properties->Marquee.Rect.GetSize() * PinnedOwningWidget->GetZoomAmount(), FSlateLayoutTransform(PinnedOwningWidget->GraphCoordToPanelCoord(Properties->Marquee.Rect.GetUpperLeft()))),
				FAppStyle::GetBrush(TEXT("MarqueeSelection"))
			);
		}
	}

	return LayerId;
}

void URectangleSelectTool::SelectWidgetsAffectedByMarquee()
{
	TSharedPtr<SDesignerView> PinnedOwningWidget = OwningWidget.Pin();
	TSharedPtr<FWidgetBlueprintEditor> WidgetBlueprintEditor = OwningToolkit.Pin();
	if (PinnedOwningWidget && WidgetBlueprintEditor)
	{
		TArray<UWidget*> WidgetsInTree;
		WidgetBlueprintEditor->GetPreview()->WidgetTree->GetAllWidgets(WidgetsInTree);

		// We are only interested in the delta from the previous selection
		// We can get two rectangles that any widgets in this selection delta must intersect 
		// by comparing the previous selection's endpoint with our current endpoint, 
		// and projecting those points onto the axis-aligned lines centered at the selection start

		FVector2D StartPoint = Properties->Marquee.Rect.StartPoint;
		FVector2D OldEndPoint = PrevSelectionEndPoint;
		FVector2D NewEndPoint = Properties->Marquee.Rect.EndPoint;

		float SelectionDeltaX = FMath::Square(NewEndPoint.X - StartPoint.X) - FMath::Square(OldEndPoint.X - StartPoint.X);
		float SelectionDeltaY = FMath::Square(NewEndPoint.Y - StartPoint.Y) - FMath::Square(OldEndPoint.Y - StartPoint.Y);

		// If our selection is shrinking, use previous coordinate pair value to ensure a collision when testing for deselection

		FMarqueeRect RectDeltaX;
		RectDeltaX.StartPoint = FVector2f(OldEndPoint.X, StartPoint.Y);
		RectDeltaX.EndPoint = FVector2f(NewEndPoint.X, SelectionDeltaY > 0 ? NewEndPoint.Y : OldEndPoint.Y);

		FMarqueeRect RectDeltaY;
		RectDeltaY.StartPoint = FVector2f(StartPoint.X, OldEndPoint.Y);
		RectDeltaY.EndPoint = FVector2f(SelectionDeltaX > 0 ? NewEndPoint.X : OldEndPoint.X, NewEndPoint.Y);

		FMarqueeRect PrevSelectionRect;
		PrevSelectionRect.StartPoint = StartPoint;
		PrevSelectionRect.EndPoint = PrevSelectionEndPoint;

		bool bToggledSelection = false;
		TSet<FWidgetReference> ToggleSelectionSet;
		for (UWidget* Widget : WidgetsInTree)
		{
			if (Widget && !Widget->IsRooted())
			{
				const FVector2D Position = PinnedOwningWidget->PanelCoordToGraphCoord(PinnedOwningWidget->GetTickSpaceGeometry().AbsoluteToLocal(Widget->GetTickSpaceGeometry().GetAbsolutePosition()));
				const FVector2D Size = Widget->GetPaintSpaceGeometry().GetAbsoluteSize() * PinnedOwningWidget->GetPreviewDPIScale() / PinnedOwningWidget->GetZoomAmount();

				if (Size.X > 0.f && Size.Y > 0.f)
				{
					const FSlateRect WidgetPaintArea(Position.X, Position.Y, Position.X + Size.X, Position.Y + Size.Y);

					// Intersecting with delta rect, toggle selection appropriately if needed
					auto ToggleSelection = [&](const FMarqueeRect& DeltaRect, float SelectionDelta)
					{
						const bool bCanSelectionChange = SelectionDelta && FSlateRect::DoRectanglesIntersect(DeltaRect.ToSlateRect(), WidgetPaintArea);
						if (bCanSelectionChange)
						{
							if (SelectionDelta > 0 && FSlateRect::IsRectangleContained(Properties->Marquee.Rect.ToSlateRect(), WidgetPaintArea))
							{
								ToggleSelectionSet.Add(WidgetBlueprintEditor->GetReferenceFromPreview(Widget));
								return true;
							}
							else if (SelectionDelta < 0 && FSlateRect::IsRectangleContained(PrevSelectionRect.ToSlateRect(), WidgetPaintArea))
							{
								ToggleSelectionSet.Add(WidgetBlueprintEditor->GetReferenceFromPreview(Widget));
								return true;
							}
						}
						return false;
					};

					bToggledSelection |= ToggleSelection(RectDeltaX, SelectionDeltaX) || ToggleSelection(RectDeltaY, SelectionDeltaY);
				}
			}
		}

		if (bToggledSelection)
		{
			WidgetBlueprintEditor->SelectWidgets(ToggleSelectionSet, true /*bool bAppendOrToggle - true implies toggle */);
		}

		PrevSelectionEndPoint = NewEndPoint;
	}
}

#undef LOCTEXT_NAMESPACE
