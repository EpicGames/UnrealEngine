// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetEditorToolPaletteCommands.h"
#include "WidgetEditorToolPaletteStyle.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Settings/CreateWidgetToolSettings.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "WidgetEditorToolPaletteCommands"

FWidgetEditorToolPaletteCommands::FWidgetEditorToolPaletteCommands()
	: TCommands<FWidgetEditorToolPaletteCommands>(
		TEXT("WidgetEditorToolPaletteCommands"), 
		NSLOCTEXT("Contexts", "WidgetToolPalette", "Widget Tool Palette"), 
		NAME_None, 
		FWidgetEditorToolPaletteStyle::Get()->GetStyleSetName())
{}

void FWidgetEditorToolPaletteCommands::RegisterCommands()
{
	UI_COMMAND(ToggleWidgetEditorToolPalette, "Default Tools", "Standard UMG tools palette.", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(DefaultSelectTool, "Select", "Select", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::A));
	UI_COMMAND(BeginRectangleSelectTool, "Marquee", "Marquee", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::M));

	// Create commands defined in settings
	const UCreateWidgetToolSettings* Settings = GetDefault<UCreateWidgetToolSettings>();

	for (const FCreateWidgetStackInfo& CreateWidgetStack : Settings->CreateWidgetStacks)
	{
		TSharedPtr<FUICommandInfo> StackCommand;

		FUICommandInfo::MakeCommandInfo(
			this->AsShared(),
			StackCommand,
			FName(CreateWidgetStack.DisplayName),
			FText::FromString(CreateWidgetStack.DisplayName), // @TODO: how does this localize?
			FText::FromString(CreateWidgetStack.DisplayName),
			FSlateIcon(),
			EUserInterfaceActionType::ToggleButton,
			FInputChord());

		CreateWidgetToolStacks.Emplace(CreateWidgetStack.DisplayName, StackCommand);

		for (const FCreateWidgetToolInfo& CreateWidgetToolInfo : CreateWidgetStack.WidgetToolInfos)
		{
			check(CreateWidgetToolInfo.WidgetClass);

			TSharedPtr<FUICommandInfo> ToolCommand;

			FText DisplayName = CreateWidgetToolInfo.DisplayName.IsEmpty() 
				? CreateWidgetToolInfo.WidgetClass->GetDisplayNameText()
				: FText::FromString(CreateWidgetToolInfo.DisplayName);

			FUICommandInfo::MakeCommandInfo(
				this->AsShared(),
				ToolCommand,
				FName(DisplayName.ToString()),
				DisplayName, 
				DisplayName,
				FSlateIconFinder::FindIconForClass(CreateWidgetToolInfo.WidgetClass),
				EUserInterfaceActionType::ToggleButton,
				CreateWidgetToolInfo.WidgetHotkey);

			CreateWidgetTools.Emplace(DisplayName.ToString(), ToolCommand);
		}
	}
}

const FWidgetEditorToolPaletteCommands& FWidgetEditorToolPaletteCommands::Get()
{
	return TCommands<FWidgetEditorToolPaletteCommands>::Get();
}

#undef LOCTEXT_NAMESPACE
