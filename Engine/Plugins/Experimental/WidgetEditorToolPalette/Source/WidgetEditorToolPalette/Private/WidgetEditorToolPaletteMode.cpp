// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetEditorToolPaletteMode.h"
#include "WidgetEditorToolPaletteCommands.h"
#include "WidgetEditorToolPaletteToolkit.h"
#include "ContextObjectStore.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "ToolContexts/ToolStackContext.h"
#include "ToolTargetManager.h"
#include "DefaultTools/RectangleSelectTool.h"
#include "DefaultTools/CreateWidgetTool.h"
#include "BaseTools/SingleClickTool.h"
#include "Settings/CreateWidgetToolSettings.h"

#define LOCTEXT_NAMESPACE "WidgetEditorToolPaletteMode"

const FEditorModeID UWidgetEditorToolPaletteMode::Id("WidgetEditorToolPaletteMode");

UWidgetEditorToolPaletteMode::UWidgetEditorToolPaletteMode()
{
	bool bVisibleInLevelEditor = false;
	Info = FEditorModeInfo(
		Id, 
		LOCTEXT("WidgetEditorToolPaletteMode", "WidgetTools"), 
		FSlateIcon("WidgetEditorToolPaletteStyle", "WidgetEditorToolPaletteCommands.DefaultSelectTool"), 
		bVisibleInLevelEditor);
}

void UWidgetEditorToolPaletteMode::Enter()
{
	UEdMode::Enter();
	
	const FWidgetEditorToolPaletteCommands& ToolPaletteCommands = FWidgetEditorToolPaletteCommands::Get();

	USingleClickToolBuilder* SingleClickToolBuilder = NewObject<USingleClickToolBuilder>();
	RegisterTool(ToolPaletteCommands.DefaultSelectTool, TEXT("DefaultCursor"), SingleClickToolBuilder);

	URectangleSelectToolBuilder* RectangleSelectToolBuilder = NewObject<URectangleSelectToolBuilder>();
	RegisterTool(ToolPaletteCommands.BeginRectangleSelectTool, TEXT("BeginRectangleSelectTool"), RectangleSelectToolBuilder);

	// Add a tool stack context for tool stack support, tools will automatically be unregistered via destructor when context store is freed
	UEditorInteractiveToolsContext* EdToolsContext = GetInteractiveToolsContext(EToolsContextScope::EdMode);
	if (EdToolsContext && EdToolsContext->ContextObjectStore && EdToolsContext->ToolManager)
	{
		UToolStackContext* ToolStackContext = NewObject<UToolStackContext>(EdToolsContext->ToolManager);
		EdToolsContext->ContextObjectStore->AddContextObject(ToolStackContext);
		ToolStackContext->EdMode = this;
		ToolStackContext->ToolContextScope = EToolsContextScope::EdMode;

		// Create tool stacks defined in settings
		const UCreateWidgetToolSettings* Settings = GetDefault<UCreateWidgetToolSettings>();
		for (const FCreateWidgetStackInfo& CreateWidgetStack : Settings->CreateWidgetStacks)
		{
			const TSharedPtr<FUICommandInfo>& StackCommand = ToolPaletteCommands.CreateWidgetToolStacks[CreateWidgetStack.DisplayName];
			ToolStackContext->RegisterToolStack(StackCommand, CreateWidgetStack.DisplayName);

			for (const FCreateWidgetToolInfo& CreateWidgetToolInfo : CreateWidgetStack.WidgetToolInfos)
			{
				check(CreateWidgetToolInfo.WidgetClass);

				FText DisplayName = CreateWidgetToolInfo.DisplayName.IsEmpty()
					? CreateWidgetToolInfo.WidgetClass->GetDisplayNameText()
					: FText::FromString(CreateWidgetToolInfo.DisplayName);

				const TSharedPtr<FUICommandInfo>& ToolCommand = ToolPaletteCommands.CreateWidgetTools[DisplayName.ToString()];

				UCreateWidgetToolBuilder* CreateWidgetToolBuilder = NewObject<UCreateWidgetToolBuilder>(GetTransientPackage(), CreateWidgetToolInfo.CreateWidgetToolBuilder);
				CreateWidgetToolBuilder->WidgetClass = CreateWidgetToolInfo.WidgetClass;

				ToolStackContext->AddToolToStack(ToolCommand, CreateWidgetStack.DisplayName, DisplayName.ToString(), CreateWidgetToolBuilder);
			}
		}
	}

	GetToolManager()->SelectActiveToolType(EToolSide::Mouse, TEXT("DefaultCursor"));
}

bool UWidgetEditorToolPaletteMode::UsesToolkits() const
{ 
	return true; 
}

void UWidgetEditorToolPaletteMode::CreateToolkit()
{
	Toolkit = MakeShareable(new FWidgetEditorToolPaletteToolkit);
}

#undef LOCTEXT_NAMESPACE
