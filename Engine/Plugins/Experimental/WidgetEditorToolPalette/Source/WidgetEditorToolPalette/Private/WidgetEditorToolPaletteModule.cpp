// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetEditorToolPaletteModule.h"
#include "WidgetEditorToolPaletteCommands.h"
#include "WidgetEditorToolPaletteMode.h"
#include "WidgetEditorToolPaletteStyle.h"
#include "ToolMenus.h"
#include "UMGEditorModule.h"
#include "WidgetBlueprintEditor.h"
#include "EditorModeManager.h"

#define LOCTEXT_NAMESPACE "FWidgetEditorToolPaletteModule"

void FWidgetEditorToolPaletteModule::StartupModule()
{
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FWidgetEditorToolPaletteModule::OnPostEngineInit);

	// Register command for toolbar action
	auto ExtendUMGEditorToolbar = [this](const TSharedRef<FUICommandList> InCommandList, TSharedRef<FWidgetBlueprintEditor> Editor) -> TSharedRef<FExtender>
	{
		// Note: actual UI extension done via 'ToolPaletteCommands'
		TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);
		TWeakPtr<FWidgetBlueprintEditor> Ptr(Editor);

		InCommandList->MapAction(FWidgetEditorToolPaletteCommands::Get().ToggleWidgetEditorToolPalette,
			FExecuteAction::CreateRaw(this, &FWidgetEditorToolPaletteModule::OnToggleWidgetEditorToolPaletteMode, Ptr),
			FCanExecuteAction(),
			FIsActionChecked::CreateRaw(this, &FWidgetEditorToolPaletteModule::IsWidgetEditorToolPaletteModeActive, Ptr));
		Editor->ToolPaletteCommands.Add(FWidgetEditorToolPaletteCommands::Get().ToggleWidgetEditorToolPalette);

		return ToolbarExtender.ToSharedRef();
	};

	IUMGEditorModule& UMGEditorModule = FModuleManager::LoadModuleChecked<IUMGEditorModule>("UMGEditor");
	UMGEditorModule.AddWidgetEditorToolbarExtender(IUMGEditorModule::FWidgetEditorToolbarExtender::CreateLambda(ExtendUMGEditorToolbar));
}


void FWidgetEditorToolPaletteModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	UToolMenus::UnregisterOwner(this);
	FWidgetEditorToolPaletteCommands::Unregister();
	FWidgetEditorToolPaletteStyle::Shutdown();
}

bool FWidgetEditorToolPaletteModule::IsWidgetEditorToolPaletteModeActive(TWeakPtr<FWidgetBlueprintEditor> InEditor) const
{
	TSharedPtr<FWidgetBlueprintEditor> PinnedEditor = InEditor.Pin();
	return PinnedEditor.IsValid() && PinnedEditor->GetEditorModeManager().IsModeActive(UWidgetEditorToolPaletteMode::Id);
}


void FWidgetEditorToolPaletteModule::OnToggleWidgetEditorToolPaletteMode(TWeakPtr<FWidgetBlueprintEditor> InEditor)
{
	TSharedPtr<FWidgetBlueprintEditor> PinnedEditor = InEditor.Pin();

	if (PinnedEditor.IsValid())
	{
		if (!IsWidgetEditorToolPaletteModeActive(InEditor))
		{
			PinnedEditor->GetEditorModeManager().ActivateMode(UWidgetEditorToolPaletteMode::Id, true);
		}
		else
		{
			PinnedEditor->GetEditorModeManager().ActivateDefaultMode();
		}
	}
}

void FWidgetEditorToolPaletteModule::OnPostEngineInit()
{
	// Register slate style overrides
	FWidgetEditorToolPaletteStyle::Initialize();

	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
	FWidgetEditorToolPaletteCommands::Register();
}


#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FWidgetEditorToolPaletteModule, WidgetEditorToolPalette)

