// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetEditorToolPaletteStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Interfaces/IPluginManager.h"
#include "SlateOptMacros.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/AppStyle.h"

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( FWidgetEditorToolPaletteStyle::InContent( RelativePath, ".png" ), __VA_ARGS__ )
#define RootToContentDir StyleSet->RootToContentDir

FString FWidgetEditorToolPaletteStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
{
	static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("WidgetEditorToolPalette"))->GetContentDir();
	return (ContentDir / RelativePath) + Extension;
}

TSharedPtr< FSlateStyleSet > FWidgetEditorToolPaletteStyle::StyleSet = nullptr;
TSharedPtr< class ISlateStyle > FWidgetEditorToolPaletteStyle::Get() { return StyleSet; }

FName FWidgetEditorToolPaletteStyle::GetStyleSetName()
{
	static FName ModelingToolsStyleName(TEXT("WidgetEditorToolPaletteStyle"));
	return ModelingToolsStyleName;
}

const FSlateBrush* FWidgetEditorToolPaletteStyle::GetBrush(FName PropertyName, const ANSICHAR* Specifier)
{
	return Get()->GetBrush(PropertyName, Specifier);
}


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FWidgetEditorToolPaletteStyle::Initialize()
{
	// Const icon sizes
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon28x28(28.0f, 28.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon120(120.0f, 120.0f);

	// Only register once
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShareable(new FSlateStyleSet(GetStyleSetName()));
	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const FTextBlockStyle& NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");

	// Tool Manager icons
	{
		// if (FCoreStyle::IsStarshipStyle())
		// {
		// 	StyleSet->Set("LevelEditor.ModelingToolsMode", new IMAGE_BRUSH_SVG("Starship/geometry", FVector2D(20.0f, 20.0f)));
		// }
		// else
		// {
		// 	StyleSet->Set("LevelEditor.ModelingToolsMode", new IMAGE_PLUGIN_BRUSH("Icons/icon_WidgetEditorToolPalette", FVector2D(40.0f, 40.0f)));
		// 	StyleSet->Set("LevelEditor.ModelingToolsMode.Small", new IMAGE_PLUGIN_BRUSH("Icons/icon_WidgetEditorToolPalette", FVector2D(20.0f, 20.0f)));
		// }

		StyleSet->Set("WidgetEditorToolPaletteCommands.ToggleWidgetEditorToolPalette", new IMAGE_BRUSH("Icons/GeneralTools/Paint_40x", Icon20x20));
		StyleSet->Set("WidgetEditorToolPaletteCommands.ToggleWidgetEditorToolPalette.Small", new IMAGE_BRUSH("Icons/GeneralTools/Paint_40x", Icon20x20));

		StyleSet->Set("WidgetEditorToolPaletteCommands.DefaultSelectTool", 		new IMAGE_BRUSH("Icons/GeneralTools/Select_40x", Icon20x20));
		StyleSet->Set("WidgetEditorToolPaletteCommands.DefaultSelectTool.Small", new IMAGE_BRUSH("Icons/GeneralTools/Select_40x", Icon20x20));
		StyleSet->Set("WidgetEditorToolPaletteCommands.BeginRectangleSelectTool", new IMAGE_BRUSH_SVG("Icons/GeneralTools/Marque", Icon20x20));
		StyleSet->Set("WidgetEditorToolPaletteCommands.BeginRectangleSelectTool.Small", new IMAGE_BRUSH_SVG("Icons/GeneralTools/Marque", Icon20x20));
	}

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
};

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef IMAGE_PLUGIN_BRUSH
#undef RootToContentDir

void FWidgetEditorToolPaletteStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}
