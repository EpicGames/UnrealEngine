// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetEditorToolPaletteToolkit.h"
#include "WidgetEditorToolPaletteCommands.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

#include "Tools/UEdMode.h"
#include "InteractiveTool.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "InteractiveToolManager.h"
#include "Widgets/Text/STextBlock.h"
#include "IDetailsView.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/SToolBarStackButtonBlock.h"
#include "Settings/CreateWidgetToolSettings.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "WidgetEditorToolPalette"

static const FName WidgetToolPaletteTabName(TEXT("WidgetToolPalette"));

void FWidgetEditorToolPaletteToolkit::Init(
	const TSharedPtr<IToolkitHost>& InToolkitHost,
	TWeakObjectPtr<UEdMode> InOwningMode )
{
	FModeToolkit::Init(InToolkitHost, InOwningMode);

	ActiveToolName = FText::GetEmpty();
	ActiveToolMessage = FText::GetEmpty();
	SetCurrentPalette(WidgetToolPaletteTabName);
}

FName FWidgetEditorToolPaletteToolkit::GetToolkitFName() const
{
	return FName("WidgetToolPaletteToolkit");
}

FText FWidgetEditorToolPaletteToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("DisplayName", "WidgetToolPaletteToolkit");
}

void FWidgetEditorToolPaletteToolkit::GetToolPaletteNames(TArray<FName>& PaletteNames) const
{
	PaletteNames = { WidgetToolPaletteTabName };
}

FText FWidgetEditorToolPaletteToolkit::GetToolPaletteDisplayName(FName PaletteName) const
{
	return FText::FromName(PaletteName);
}

void FWidgetEditorToolPaletteToolkit::BuildToolPalette(FName PaletteName, class FToolBarBuilder& ToolbarBuilder)
{
	const FWidgetEditorToolPaletteCommands& ToolPaletteCommands = FWidgetEditorToolPaletteCommands::Get();

	if (PaletteName == WidgetToolPaletteTabName)
	{
		ToolbarBuilder.AddToolBarButton(ToolPaletteCommands.DefaultSelectTool);
		ToolbarBuilder.AddToolBarButton(ToolPaletteCommands.BeginRectangleSelectTool);
		ToolbarBuilder.AddSeparator();

		// Create tool stacks defined in settings
		const UCreateWidgetToolSettings* Settings = GetDefault<UCreateWidgetToolSettings>();
		for (const TPair<FString, TSharedPtr<FUICommandInfo>>& CreateWidgetStack : ToolPaletteCommands.CreateWidgetToolStacks)
		{
			ToolbarBuilder.AddToolbarStackButton(CreateWidgetStack.Value);
		}
	}
}

TSharedRef<SWidget> FWidgetEditorToolPaletteToolkit::CreatePaletteWidget(TSharedPtr<FUICommandList> InCommandList, FName InToolbarCustomizationName, FName InPaletteName)
{
	FVerticalToolBarBuilder ModeToolbarBuilder(InCommandList, FMultiBoxCustomization(InToolbarCustomizationName));
	ModeToolbarBuilder.SetStyle(&FAppStyle::Get(), "VerticalToolBar");

	BuildToolPalette(InPaletteName, ModeToolbarBuilder);

	return ModeToolbarBuilder.MakeWidget();
}

FText FWidgetEditorToolPaletteToolkit::GetActiveToolDisplayName() const
{
	return ActiveToolName;
}

FText FWidgetEditorToolPaletteToolkit::GetActiveToolMessage() const
{
	return ActiveToolMessage;
}

void FWidgetEditorToolPaletteToolkit::UpdateActiveToolProperties(UInteractiveTool* Tool)
{
	if (Tool)
	{
		DetailsView->SetObjects(Tool->GetToolProperties(true));
	}
}

void FWidgetEditorToolPaletteToolkit::OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	FModeToolkit::OnToolStarted(Manager, Tool);

	Tool->OnPropertySetsModified.AddSP(this, &FWidgetEditorToolPaletteToolkit::UpdateActiveToolProperties, Tool);

	ActiveToolName = Tool->GetToolInfo().ToolDisplayName;
}


void FWidgetEditorToolPaletteToolkit::OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	FModeToolkit::OnToolEnded(Manager, Tool);

	if (Tool)
	{
		Tool->OnPropertySetsModified.RemoveAll(this);
	}

	ActiveToolName = FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
