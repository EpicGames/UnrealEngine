// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/SingleClickTool.h"
#include "BaseBehaviors/Widgets/WidgetBaseBehavior.h"
#include "InteractiveToolBuilder.h"
#include "CreateWidgetTool.generated.h"

/**
 * Builder for UCreateWidgetTool
 */
UCLASS()
class WIDGETEDITORTOOLPALETTE_API UCreateWidgetToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

	UCreateWidgetToolBuilder();

public:
	virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override { return true; }
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

public:
	/** Widget Class to be created by this tool */
	UClass* WidgetClass;
};

/**
 * Settings UObject for UCreateWidgetTool. This UClass inherits from UInteractiveToolPropertySet,
 * which provides an OnModified delegate that the Tool will listen to for changes in property values.
 */
UCLASS(Transient)
class WIDGETEDITORTOOLPALETTE_API UCreateWidgetToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	UCreateWidgetToolProperties();
};

/**
 * UCreateWidgetTool is a tool that allows for a specific widget to be created on click
 * It allows for fast widget creation via hotkeys. The specific OnClick behavior can be customized
 * by specifying a different create widget tool builder class in editor per project user settings.
 */
UCLASS()
class WIDGETEDITORTOOLPALETTE_API UCreateWidgetTool : public USingleClickTool, public IWidgetBaseBehavior
{
	GENERATED_BODY()

public:
	UCreateWidgetTool();

	virtual void SetOwningToolkit(TSharedPtr<class IToolkit> InOwningToolkit);
	virtual void SetOwningWidget(TSharedPtr<class SWidget> InOwningWidget);

	// ~Begin UInteractiveTool Interface
	virtual void Setup() override;
	// ~End UInteractiveTool Interface

	// ~Begin IWidgetBaseBehavior Interface
	virtual bool OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	// ~End IWidgetBaseBehavior Interface

public:
	/** Widget Class to be created by this tool */
	UClass* WidgetClass;

protected:
	UPROPERTY()
	TObjectPtr<UCreateWidgetToolProperties> Properties;

protected:

	// @TODO: DarenC - Find a good parent class or interface for these (as base class)
	TWeakPtr<class FWidgetBlueprintEditor> OwningToolkit;
	TWeakPtr<class SDesignerView> OwningWidget;
};