// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/ClickDragTool.h"
#include "BaseBehaviors/Widgets/WidgetBaseBehavior.h"
#include "InteractiveToolBuilder.h"
#include "MarqueeOperation.h"

#include "RectangleSelectTool.generated.h"

/**
 * Builder for URectangleSelectTool
 */
UCLASS()
class WIDGETEDITORTOOLPALETTE_API URectangleSelectToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};

/**
 * Property set for the URectangleSelectTool
 * 
 * Note: Since we don't expose this for editing via details (yet), could instead be member variables on tool.
 */
UCLASS(Transient)
class WIDGETEDITORTOOLPALETTE_API URectangleSelectProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	URectangleSelectProperties();

	/** A pending marquee operation if it's active */
	bool bIsMarqueeActive;

	/** A pending marquee operation if it's active */
	FMarqueeOperation Marquee;
};

/**
 * URectangleSelectTool is a simple marque widget select tool.
 */
UCLASS()
class WIDGETEDITORTOOLPALETTE_API URectangleSelectTool : public UInteractiveTool, public IWidgetBaseBehavior
{
	GENERATED_BODY()

public:
	virtual void SetOwningToolkit(TSharedPtr<class IToolkit> InOwningToolkit);
	virtual void SetOwningWidget(TSharedPtr<class SWidget> InOwningWidget);

	// ~Begin UInteractiveTool Interface
	virtual void Setup() override;
	// ~End UInteractiveTool Interface

	// ~Begin IWidgetBaseBehavior Interface
	virtual bool OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual bool OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual bool OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual bool OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	// ~End IWidgetBaseBehavior Interface

protected:
	/** Properties of the tool are stored here */
	UPROPERTY()
	TObjectPtr<URectangleSelectProperties> Properties;

	/** Cached last end point to optimize comparisons */
	FVector2D PrevSelectionEndPoint;

protected:

	// @TODO: DarenC - Find a good parent class or interface for these (as base class)
	TWeakPtr<class FWidgetBlueprintEditor> OwningToolkit;
	TWeakPtr<class SDesignerView> OwningWidget;

	void SelectWidgetsAffectedByMarquee();
};
