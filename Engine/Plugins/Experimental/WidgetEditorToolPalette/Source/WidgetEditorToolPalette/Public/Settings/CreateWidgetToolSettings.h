// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/InputChord.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/DeveloperSettings.h"
#include "Engine/EngineTypes.h"
#include "Components/Widget.h"
#include "DefaultTools/CreateWidgetTool.h"
#include "CreateWidgetToolSettings.generated.h"


/** Info used to populate a single create widget tool */
USTRUCT()
struct WIDGETEDITORTOOLPALETTE_API FCreateWidgetToolInfo
{
	GENERATED_BODY()

	FCreateWidgetToolInfo()
		: WidgetClass(nullptr)
		, DisplayName("")
		, WidgetHotkey(FInputChord())
		, CreateWidgetToolBuilder(UCreateWidgetToolBuilder::StaticClass())
	{
	}

	/** The widget to create when this tool is activated */
	UPROPERTY(EditAnywhere, config, Category = CreateWidgetInfo)
	TSubclassOf<UWidget> WidgetClass;

	/** Display name for this tool, if empty will use the name of the widget */
	UPROPERTY(EditAnywhere, config, Category = CreateWidgetInfo)
	FString DisplayName;

	/** The hotkey used to create this widget */
	UPROPERTY(EditAnywhere, config, Category = CreateWidgetInfo)
	FInputChord WidgetHotkey;

	/** Builder that handles creation of tool for this widget */
	UPROPERTY(EditAnywhere, config, AdvancedDisplay, Category = CreateWidgetInfo)
	TSubclassOf<UCreateWidgetToolBuilder> CreateWidgetToolBuilder;
};

/** Info used to populate a create widget tool stack */
USTRUCT()
struct WIDGETEDITORTOOLPALETTE_API FCreateWidgetStackInfo
{
	GENERATED_BODY()

	/** Display name for this tool stack */
	UPROPERTY(EditAnywhere, config, Category = CreateWidgetInfo)
	FString DisplayName;

	/** Info for each widget that can be created by this stack */
	UPROPERTY(EditAnywhere, config, Category = CreateWidgetInfo)
	TArray<FCreateWidgetToolInfo> WidgetToolInfos;
};

/**
 * Allows for create widget tool layout to be defined via settings.
 */
UCLASS(config=WidgetEditorToolPalette, autoexpandcategories=(Keybinds))
class WIDGETEDITORTOOLPALETTE_API UCreateWidgetToolSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:

	virtual FName GetContainerName() const { return FName("Editor"); }
	virtual FName GetCategoryName() const { return FName("Plugins"); }
	virtual FName GetSectionName() const { return FName("CreateWidgetHotkeys"); }


public:

	/** List of tool stacks to create */
	UPROPERTY(EditAnywhere, config, Category=Startup)
	TArray<FCreateWidgetStackInfo> CreateWidgetStacks;
};
