// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class WIDGETEDITORTOOLPALETTE_API FWidgetEditorToolPaletteCommands : public TCommands<FWidgetEditorToolPaletteCommands>
{
public:
	FWidgetEditorToolPaletteCommands();
	void RegisterCommands() override;
	static const FWidgetEditorToolPaletteCommands& Get();

	/** Command to toggle tool palette in widget editor */
	TSharedPtr<FUICommandInfo> ToggleWidgetEditorToolPalette;

	TSharedPtr<FUICommandInfo> DefaultSelectTool;
	TSharedPtr<FUICommandInfo> BeginRectangleSelectTool;

	TMap<FString, TSharedPtr<FUICommandInfo>> CreateWidgetToolStacks;
	TMap<FString, TSharedPtr<FUICommandInfo>> CreateWidgetTools;
};

