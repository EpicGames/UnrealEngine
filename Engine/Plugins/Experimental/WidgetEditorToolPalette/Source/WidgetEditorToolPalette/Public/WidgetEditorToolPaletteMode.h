// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/LegacyEdModeWidgetHelpers.h"
#include "Tools/Modes.h"
#include "WidgetEditorToolPaletteMode.generated.h"

class SBorder;

UCLASS(Transient)
class UWidgetEditorToolPaletteMode : public UBaseLegacyWidgetEdMode
{
	GENERATED_BODY()

public:

	const static FEditorModeID Id;

protected:

	UWidgetEditorToolPaletteMode();

	void Enter() override;

	bool UsesToolkits() const override; 
	void CreateToolkit() override;
	
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }

};
