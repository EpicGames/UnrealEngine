// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class FWidgetBlueprintEditor;
class SDockTab;

class FWidgetEditorToolPaletteModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	bool IsWidgetEditorToolPaletteModeActive(TWeakPtr<FWidgetBlueprintEditor> InEditor) const;
	void OnToggleWidgetEditorToolPaletteMode(TWeakPtr<FWidgetBlueprintEditor> InEditor);

	void OnPostEngineInit();

};

