// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/BaseToolkit.h"

class STextBlock;

class FWidgetEditorToolPaletteToolkit : public FModeToolkit
{
public:

	void Init( const TSharedPtr<IToolkitHost>& InToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode ) override;

	//~ Begin IToolkit interface
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	//~ End IToolkit interface

	//~ Begin FModeToolkit interface
	virtual void GetToolPaletteNames(TArray<FName>& InPaletteName) const override;
	virtual FText GetToolPaletteDisplayName(FName PaletteName) const override;
	virtual void BuildToolPalette(FName PaletteName, class FToolBarBuilder& ToolbarBuilder) override;
	virtual TSharedRef<SWidget> CreatePaletteWidget(TSharedPtr<FUICommandList> InCommandList, FName InToolbarCustomizationName, FName InPaletteName);
	virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	virtual void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	//~ End FModeToolkit interface

private:

	void UpdateActiveToolProperties(UInteractiveTool* Tool);

	//~ Begin FModeToolkit interface
	virtual FText GetActiveToolDisplayName() const override;
	virtual FText GetActiveToolMessage() const override;
	//~ End FModeToolkit interface

	FText ActiveToolName;
	FText ActiveToolMessage;

};



