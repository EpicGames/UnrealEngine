// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WidgetEditorToolPalette : ModuleRules
{
	public WidgetEditorToolPalette(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{			
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"InputCore",
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"EditorInteractiveToolsFramework",
				"EditorStyle",
				"EditorFramework",
				"GraphEditor",
				"ContentBrowser",
				"LevelEditor",
				"Projects",
				"InteractiveToolsFramework",
				"Slate",
				"SlateCore",
				"UMG",
				"UMGEditor",
				"ToolMenus",
				"UnrealEd",
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);

	}
}
