{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Tool Palette for the Widget Editor",
	"Description": "A set of tools to enhance UMG creation UX",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Hidden": false,
	"Modules": [
		{
			"Name": "WidgetEditorToolPalette",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64" ]
		}
	]
}
